{
	"FileVersion" : 1,
	"FriendlyName" : "RenderDoc Plugin",
	"Version" : 1,
	"VersionName" : "1.0",
	"CreatedBy" : "Fredrik Lindh (Temaran)",
	"CreatedByURL" : "https://github.com/Temaran/UE4RenderDocPlugin",
	"DocsURL" : "https://renderdoc.org/docs/index.html",
	"Description" : "RenderDoc graphics debugger/profiler integration.",
	"Category" : "Rendering",
	"EnabledByDefault" : true,
	"Modules" :
	[
		{
			"Name": "RenderDocPlugin",
			"Type": "DeveloperTool",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		}
	]
}
