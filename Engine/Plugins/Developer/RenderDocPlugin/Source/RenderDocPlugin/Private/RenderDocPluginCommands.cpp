// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "RenderDocPluginCommands.h"

#define LOCTEXT_NAMESPACE "RenderDocPlugin"
 
void FRenderDocPluginCommands::RegisterCommands()
{
	UI_COMMAND(CaptureFrame, "Capture Frame", "Captures the next frame and launches the renderdoc UI (Alt+F12)", EUserInterfaceActionType::Button, FInputChord(EKeys::F12, EModifierKey::Alt));
}

#undef LOCTEXT_NAMESPACE

#endif //WITH_EDITOR
