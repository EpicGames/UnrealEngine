// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "RenderDocPluginStyle.h"
#include "Framework/Commands/Commands.h"

class FRenderDocPluginCommands : public TCommands<FRenderDocPluginCommands>
{
public:
	FRenderDocPluginCommands()
		: TCommands<FRenderDocPluginCommands>(TEXT("RenderDocPlugin"), NSLOCTEXT("Contexts", "RenderDocPlugin", "RenderDoc Plugin"), NAME_None, FRenderDocPluginStyle::Get()->GetStyleSetName())
	{			
	}

	virtual void RegisterCommands() override;

	TSharedPtr<class FUICommandInfo> CaptureFrame;
};

#endif
