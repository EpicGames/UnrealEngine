// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderDocPluginLoader.h"
#include "RenderDocPluginModule.h"
#include "RenderDocPluginSettings.h"

#include "Internationalization/Internationalization.h"

#include "DesktopPlatformModule.h"
#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/ConfigUtilities.h"
#include "RHI.h"

#if PLATFORM_LINUX
#include <unistd.h> // for readlink
#endif

#define LOCTEXT_NAMESPACE "RenderDocPlugin"

static TAutoConsoleVariable<int32> CVarRenderDocAutoAttach(
	TEXT("renderdoc.AutoAttach"),
	0,
	TEXT("RenderDoc will attach on startup."));

static TAutoConsoleVariable<int32> CVarRenderDocEnableCrashHandler(
	TEXT("renderdoc.EnableCrashHandler"),
	0,
	TEXT("0 - Crash handling is completely delegated to the engine. ")
	TEXT("1 - The RenderDoc crash handler will be used (Only use this if you know the problem is with RenderDoc and you want to notify the RenderDoc developers!)."));

// On Linux, the renderdoc tree is set up like:
//   /home/user/dev/renderdoc_1.18/bin/qrenderdoc
//   /home/user/dev/renderdoc_1.18/lib/librenderdoc.so
// CVarRenderDocBinaryPath would be `/home/user/dev/renderdoc_1.18/bin' in this case
static TAutoConsoleVariable<FString> CVarRenderDocBinaryPath(
	TEXT("renderdoc.BinaryPath"),
	TEXT(""),
	TEXT("Path to the main RenderDoc executable to use."));

#if PLATFORM_LINUX
	static const FString RenderDocDllName = TEXT("librenderdoc.so");
#else
	static const FString RenderDocDllName = TEXT("renderdoc.dll");
#endif

static void* LoadAndCheckRenderDocLibrary(FRenderDocPluginLoader::RENDERDOC_API_CONTEXT*& RenderDocAPI, const FString& RenderDocPath)
{
	check(nullptr == RenderDocAPI);

	if (RenderDocPath.IsEmpty())
	{
		return(nullptr);
	}

	FString PathToRenderDocDLL = FPaths::Combine(*RenderDocPath, *RenderDocDllName);
	if (!FPaths::FileExists(PathToRenderDocDLL))
	{
		if (FApp::IsUnattended())
		{
			UE_LOGF(RenderDocPlugin, Display, "unable to locate RenderDoc library at: %ls", *PathToRenderDocDLL);
		}
		else
		{
			UE_LOGF(RenderDocPlugin, Warning, "unable to locate RenderDoc library at: %ls", *PathToRenderDocDLL);
		}
		return(nullptr);
	}

	UE_LOGF(RenderDocPlugin, Log, "a RenderDoc library has been located at: %ls", *PathToRenderDocDLL);

	void* RenderDocDLL = FPlatformProcess::GetDllHandle(*PathToRenderDocDLL);
	if (RenderDocDLL == nullptr)
	{
		UE_LOGF(RenderDocPlugin, Warning, "unable to dynamically load RenderDoc library");
		return(nullptr);
	}

	pRENDERDOC_GetAPI RENDERDOC_GetAPI = (pRENDERDOC_GetAPI)FPlatformProcess::GetDllExport(RenderDocDLL, TEXT("RENDERDOC_GetAPI"));
	if (RENDERDOC_GetAPI == nullptr)
	{
		UE_LOGF(RenderDocPlugin, Warning, "unable to obtain 'RENDERDOC_GetAPI' function from '%ls'. You are likely using an incompatible version of RenderDoc.", *RenderDocDllName);
		FPlatformProcess::FreeDllHandle(RenderDocDLL);
		return(nullptr);
	}

	// Version checking and reporting
	if (0 == RENDERDOC_GetAPI(eRENDERDOC_API_Version_1_0_0, (void**)&RenderDocAPI))
	{
		UE_LOGF(RenderDocPlugin, Warning, "unable to initialize RenderDoc library due to API incompatibility (plugin requires eRENDERDOC_API_Version_1_0_0).");
		FPlatformProcess::FreeDllHandle(RenderDocDLL);
		return(nullptr);
	}

	//Unregister crash handler unless the user has enabled it. This is to avoid sending unneccesary crash reports to Baldur.
	if (!CVarRenderDocEnableCrashHandler.GetValueOnAnyThread())
	{
		RenderDocAPI->UnloadCrashHandler();
	}

	int MajorVersion(0), MinorVersion(0), PatchVersion(0);
	RenderDocAPI->GetAPIVersion(&MajorVersion, &MinorVersion, &PatchVersion);
	UE_LOGF(RenderDocPlugin, Log, "RenderDoc library has been loaded (RenderDoc API v%i.%i.%i).", MajorVersion, MinorVersion, PatchVersion);

	return(RenderDocDLL);
}

#if PLATFORM_LINUX

static FString GetRenderDocLibPath(const FString& RenderDocBinaryPath)
{
	if (!RenderDocBinaryPath.IsEmpty())
	{
		// Something like "/home/user/dev/renderdoc_1.18/bin/qrenderdoc"
		FString QRenderDocBin = FPaths::Combine(RenderDocBinaryPath, TEXT("qrenderdoc"));

		if (FPaths::FileExists(QRenderDocBin))
		{
			// And then check for "/home/user/dev/renderdoc_1.18/lib/librenderdoc.so"
			FString LibRenderDocSO = FPaths::Combine(RenderDocBinaryPath, TEXT("../lib"), RenderDocDllName);

			// Not found - check if bin/qrenderdoc is a symlink.
			if (!FPaths::FileExists(LibRenderDocSO))
			{
				TArray<ANSICHAR> ProcessPath;

				ProcessPath.AddZeroed(FPlatformMisc::GetMaxPathLength() + 1);

				int32 Ret = readlink(TCHAR_TO_UTF8(*QRenderDocBin), ProcessPath.GetData(), ProcessPath.Num() - 1);
				if (Ret != -1)
				{
					LibRenderDocSO = FPaths::Combine(FPaths::GetPath(UTF8_TO_TCHAR(ProcessPath.GetData())), TEXT("../lib"), RenderDocDllName);
				}
			}

			if (FPaths::FileExists(LibRenderDocSO))
			{
				FPaths::NormalizeFilename(QRenderDocBin);
				FPaths::CollapseRelativeDirectories(QRenderDocBin);

				FPaths::NormalizeFilename(LibRenderDocSO);
				FPaths::CollapseRelativeDirectories(LibRenderDocSO);

				// Set CVarRenderDocBinaryPath to "/home/user/dev/renderdoc_1.18/bin"
				CVarRenderDocBinaryPath.AsVariable()->Set(*FPaths::GetPath(QRenderDocBin), ECVF_SetByProjectSetting);

				// Return "/home/user/dev/renderdoc_1.18/lib/librenderdoc.so"
				return LibRenderDocSO;
			}
		}
	}

	return TEXT("");
}

// Search CVarRenderDocBinaryPath and PATH env for qrenderdoc and return appropriate librenderdoc.so directory.
//
// From https://renderdoc.org/docs/getting_started/faq.html
//  On linux the binary tarball comes with files to place under /usr/share to associate RenderDoc with files.
//  This obviously also requires qrenderdoc to be available in your PATH.
static FString FindRenderDocLibPath()
{
	// Check if we have a valid path for qrenderdoc binary already set
	FString RenderDocLibPath = GetRenderDocLibPath(CVarRenderDocBinaryPath.GetValueOnAnyThread());

	if (RenderDocLibPath.IsEmpty())
	{
		TArray<FString> PathArray;
		FPlatformMisc::GetEnvironmentVariable(TEXT("PATH")).ParseIntoArray(PathArray, FPlatformMisc::GetPathVarDelimiter());

		// Iterate through all the path directories looking for "qrenderdoc" binary
		for (const FString& Path : PathArray)
		{
			RenderDocLibPath = GetRenderDocLibPath(Path);
			if (!RenderDocLibPath.IsEmpty())
			{
				break;
			}
		}
	}

	return RenderDocLibPath;
}

#endif // PLATFORM_LINUX

void FRenderDocPluginLoader::Initialize()
{
	RenderDocDLL = nullptr;
	RenderDocAPI = nullptr;

	// Since we are so early in the loading phase, we first need to load the cvars since they're not loaded at this point.
	UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("/Script/RenderDocPlugin.RenderDocPluginSettings"), *GEngineIni, ECVF_SetByProjectSetting);

	const bool bCapture = FParse::Param(FCommandLine::Get(), TEXT("AttachRenderDoc")) || (CVarRenderDocAutoAttach.GetValueOnAnyThread() != 0);
	if (!bCapture)
	{
		UE_LOGF(RenderDocPlugin, Display, "RenderDoc plugin will not be loaded. Use '-AttachRenderDoc' on the cmd line or enable 'renderdoc.AutoAttach' in the plugin settings.");
		return;
	}

	if (GUsingNullRHI)
	{
		// THIS WILL NEVER TRIGGER because of a sort of chicken-and-egg problem: RenderDoc Loader is a PostConfigInit
		// plugin, and GUsingNullRHI is only initialized properly between PostConfigInit and PreLoadingScreen phases.
		// (nevertheless, keep this comment around for future iterations of UE)
		UE_LOGF(RenderDocPlugin, Display, "RenderDoc plugin will not be loaded because a null RHI (Cook Server, perhaps) is being used.");
		return;
	}

	// Look for a renderdoc.dll somewhere in the system:
	UE_LOGF(RenderDocPlugin, Log, "locating RenderDoc library (%ls)...", *RenderDocDllName);

	// 1) Get binary path  from CVar:
	RenderDocDLL = LoadAndCheckRenderDocLibrary(RenderDocAPI, CVarRenderDocBinaryPath.GetValueOnAnyThread());

#if PLATFORM_WINDOWS
	// 2) Check for a RenderDoc system installation in the registry:
	if (RenderDocDLL == nullptr)
	{
		FString RenderDocPath;
		FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Classes\\RenderDoc.RDCCapture.1\\DefaultIcon\\"), TEXT(""), RenderDocPath);
		RenderDocPath = FPaths::GetPath(RenderDocPath);
		RenderDocDLL = LoadAndCheckRenderDocLibrary(RenderDocAPI, RenderDocPath);
		if (RenderDocDLL)
		{
			CVarRenderDocBinaryPath.AsVariable()->Set(*RenderDocPath, ECVF_SetByProjectSetting);
		}
	}
#elif PLATFORM_LINUX
	// 2) No registry, so do whatever else we can on Linux to try and find a librenderdoc.so
	if (RenderDocDLL == nullptr)
	{
		FString RenderDocLibPath = FindRenderDocLibPath();

		if (!RenderDocLibPath.IsEmpty())
		{
			RenderDocDLL = LoadAndCheckRenderDocLibrary(RenderDocAPI, FPaths::GetPath(RenderDocLibPath));
		}
	}
#endif

	//@TODO Right now Linux uses Slate for the file dialog. We obviously can't hook Vulkan if we're using Vulkan here
	// to display file dialogs. When we get a system level dialog using gtk3 (or other) we can enable this code on Linux.
#if !PLATFORM_LINUX
	// 3) Check for a RenderDoc custom installation by prompting the user:
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (RenderDocDLL == nullptr && DesktopPlatform && !FApp::IsUnattended())
	{
		//Renderdoc does not seem to be installed, but it might be built from source or downloaded by archive,
		//so prompt the user to navigate to the main exe file
		UE_LOGF(RenderDocPlugin, Log, "RenderDoc library not found; provide a custom installation location...");

#if PLATFORM_WINDOWS
		// renderdocui.exe is the old executable name, it was changed to qrenderdoc.exe in 1.0.
		// If users upgraded from pre-1.0 to 1.0+, renderdocui.exe is a redirect to qrenderdoc.exe; otherwise only the new executable exists.
		FString Filter = TEXT("RenderDoc executable|renderdocui.exe;qrenderdoc.exe");
#elif PLATFORM_LINUX
		FString Filter = TEXT("RenderDoc executable|qrenderdoc");
#endif

		FString RenderDocPath;
		TArray<FString> OutFiles;
		if (DesktopPlatform->OpenFileDialog(nullptr, TEXT("Locate main RenderDoc executable..."), TEXT(""), TEXT(""), Filter, EFileDialogFlags::None, OutFiles))
		{
			RenderDocPath = OutFiles[0];
		}

		RenderDocPath = FPaths::GetPath(RenderDocPath);
		RenderDocDLL = LoadAndCheckRenderDocLibrary(RenderDocAPI, RenderDocPath);
		if (RenderDocDLL)
		{
			CVarRenderDocBinaryPath.AsVariable()->Set(*RenderDocPath, ECVF_SetByProjectSetting);
		}
	}
#endif

	// 4) All bets are off; aborting...
	if (RenderDocDLL == nullptr)
	{
		if (FApp::IsUnattended())
		{
			UE_LOGF(RenderDocPlugin, Display, "Unable to initialize the plugin because no RenderDoc libray has been located.");
		}
		else
		{
			UE_LOGF(RenderDocPlugin, Warning, "Unable to initialize the plugin because no RenderDoc libray has been located.");
		}

#if PLATFORM_LINUX
		UE_LOGF(RenderDocPlugin, Log, "Please add qrenderdoc directory to your PATH or set 'renderdoc.BinaryPath' cvar to a valid directory.");
#endif

		return;
	}

	UE_LOGF(RenderDocPlugin, Log, "plugin has been loaded successfully.");
}

void FRenderDocPluginLoader::Release()
{
	if (GUsingNullRHI)
	{
		return;
	}

	if (RenderDocDLL)
	{
		FPlatformProcess::FreeDllHandle(RenderDocDLL);
		RenderDocDLL = nullptr;
	}

	UE_LOGF(RenderDocPlugin, Log, "plugin has been unloaded.");
}

void* FRenderDocPluginLoader::GetRenderDocLibrary()
{
	void* RenderDocDLL = nullptr;

#if PLATFORM_LINUX
	FString PathToRenderDocDLL = GetRenderDocLibPath(CVarRenderDocBinaryPath.GetValueOnAnyThread());
#else
	FString PathToRenderDocDLL = FPaths::Combine(*CVarRenderDocBinaryPath.GetValueOnAnyThread(), *RenderDocDllName);
#endif

	RenderDocDLL = FPlatformProcess::GetDllHandle(*PathToRenderDocDLL);

	return RenderDocDLL;
}

#undef LOCTEXT_NAMESPACE

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif
