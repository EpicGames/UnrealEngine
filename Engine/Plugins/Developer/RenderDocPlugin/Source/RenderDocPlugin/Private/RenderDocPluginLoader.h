// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "renderdoc_app.h"

class FRenderDocPluginLoader
{
public:
	void Initialize();
	void Release();

	typedef RENDERDOC_API_1_0_0 RENDERDOC_API_CONTEXT;

public:
	static void* GetRenderDocLibrary();

private:
	friend class FRenderDocPluginModule;
	friend class SRenderDocPluginSettingsEditorWindow;

	void* RenderDocDLL;
	RENDERDOC_API_CONTEXT* RenderDocAPI;
};

