// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderDocPluginModule.h"
#include "Framework/Application/SlateApplication.h"
#include "UnrealClient.h"
#include "IVulkanDynamicRHI.h"

#if PLATFORM_WINDOWS
#include "ID3D11DynamicRHI.h"
#include "ID3D12DynamicRHI.h"
#endif

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Async/Async.h"
#include "HAL/FileManager.h"
#include "Internationalization/Internationalization.h"
#include "Misc/ConfigCacheIni.h"
#include "RenderDocPluginNotification.h"
#include "RenderDocPluginSettings.h"
#include "RendererInterface.h"
#include "RenderingThread.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Interfaces/IMainFrameModule.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "LevelEditor.h"
#include "SRenderDocPluginEditorExtension.h"
#endif // WITH_EDITOR

DEFINE_LOG_CATEGORY(RenderDocPlugin);

#define LOCTEXT_NAMESPACE "RenderDocPlugin"

static TAutoConsoleVariable<int32> CVarRenderDocCaptureAllActivity(
	TEXT("renderdoc.CaptureAllActivity"),
	0,
	TEXT("0 - RenderDoc will only capture data from the current viewport. ")
	TEXT("1 - RenderDoc will capture all activity, in all viewports and editor windows for the entire frame."));
static TAutoConsoleVariable<int32> CVarRenderDocCaptureCallstacks(
	TEXT("renderdoc.CaptureCallstacks"),
	1,
	TEXT("0 - Callstacks will not be captured by RenderDoc. ")
	TEXT("1 - Capture callstacks for each API call."));
static TAutoConsoleVariable<int32> CVarRenderDocReferenceAllResources(
	TEXT("renderdoc.ReferenceAllResources"),
	0,
	TEXT("0 - Only include resources that are actually used. ")
	TEXT("1 - Include all rendering resources in the capture, even those that have not been used during the frame. ")
	TEXT("Please note that doing this will significantly increase capture size."));
static TAutoConsoleVariable<int32> CVarRenderDocSaveAllInitials(
	TEXT("renderdoc.SaveAllInitials"),
	0,
	TEXT("0 - Disregard initial states of resources. ")
	TEXT("1 - Always capture the initial state of all rendering resources. ")
	TEXT("Please note that doing this will significantly increase capture size."));
static TAutoConsoleVariable<int32> CVarRenderDocCaptureDelayInSeconds(
	TEXT("renderdoc.CaptureDelayInSeconds"),
	1,
	TEXT("0 - Capture delay's unit is in frames.")
	TEXT("1 - Capture delay's unit is in seconds."));
static TAutoConsoleVariable<int32> CVarRenderDocCaptureDelay(
	TEXT("renderdoc.CaptureDelay"),
	0,
	TEXT("If > 0, RenderDoc will trigger the capture only after this amount of time (or frames, if CaptureDelayInSeconds is false) has passed."));
static TAutoConsoleVariable<int32> CVarRenderDocCaptureFrameCount(
	TEXT("renderdoc.CaptureFrameCount"),
	0,
	TEXT("If > 0, the RenderDoc capture will encompass more than a single frame. Note: this implies that all activity in all viewports and editor windows will be captured (i.e. same as CaptureAllActivity)"));

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Helper classes
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if PLATFORM_LINUX

typedef void *HWND;
static HWND GetActiveWindow()
{
	return nullptr;
}

#endif // PLATFORM_LINUX

struct FRenderDocAsyncGraphTask : public FAsyncGraphTaskBase
{
	ENamedThreads::Type TargetThread;
	TFunction<void()> TheTask;

	FRenderDocAsyncGraphTask(ENamedThreads::Type Thread, TFunction<void()>&& Task) : TargetThread(Thread), TheTask(MoveTemp(Task)) { }
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) { TheTask(); }
	ENamedThreads::Type GetDesiredThread() { return(TargetThread); }
};

class FRenderDocFrameCapturer
{
public:
	static RENDERDOC_DevicePointer GetRenderdocDevicePointer()
	{
#ifndef RENDERDOC_DEVICEPOINTER_FROM_VKINSTANCE
#define RENDERDOC_DEVICEPOINTER_FROM_VKINSTANCE(inst) (*((void **)(inst)))
#endif

		RENDERDOC_DevicePointer Device {};

		switch (RHIGetInterfaceType())
		{
	#if PLATFORM_WINDOWS
		case ERHIInterfaceType::D3D12: Device = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU(); break;
		case ERHIInterfaceType::D3D11: Device = GetID3D11DynamicRHI()->RHIGetDevice(); break;
	#endif

		case ERHIInterfaceType::Vulkan:
			Device = GetIVulkanDynamicRHI()->RHIGetVkInstance();
			Device = RENDERDOC_DEVICEPOINTER_FROM_VKINSTANCE(Device);
			break;
		}

		return Device;

	}

	static void BeginFrameCapture(HWND WindowHandle, FRenderDocPluginLoader::RENDERDOC_API_CONTEXT* RenderDocAPI)
	{
		UE_GEmitDrawEvents_BeforeCapture = GetEmitDrawEvents();
		SetEmitDrawEvents(true);
		RenderDocAPI->StartFrameCapture(GetRenderdocDevicePointer(), WindowHandle);
	}

	static uint32 EndFrameCapture(FRHICommandListImmediate& RHICmdList, HWND WindowHandle, FRenderDocPluginLoader::RENDERDOC_API_CONTEXT* RenderDocAPI)
	{
		RHICmdList.SubmitAndBlockUntilGPUIdle();

		uint32 Result = RenderDocAPI->EndFrameCapture(GetRenderdocDevicePointer(), WindowHandle);
		SetEmitDrawEvents(UE_GEmitDrawEvents_BeforeCapture);
		return Result;
	}

	static FString MakeRenderDocCaptureFilePath(FString const& InFileName)
	{
		if (InFileName.IsEmpty())
		{
			return InFileName;
		}

		const bool bAbsoluteFileName = !FPaths::IsRelative(InFileName);
		FString FileName = bAbsoluteFileName ? InFileName : FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir() / FString("RenderDocCaptures") / InFileName);
		FileName = FPaths::SetExtension(FileName, "rdc");
		FPaths::MakePlatformFilename(FileName);
		return FileName;
	}

	static void SaveAndLaunch(FRenderDocPluginModule* Plugin, uint32 Flags, const FString& InDestPath)
	{
		FString DestPath = MakeRenderDocCaptureFilePath(InDestPath);

		bool bLaunchDestPath = false;
		if (!DestPath.IsEmpty())
		{
			IPlatformFile& PlatformFileSystem = IPlatformFile::GetPlatformPhysical();

			FString NewestCapturePath = Plugin->GetNewestCapture();
			UE_LOGF(RenderDocPlugin, Log, "Copying: %ls to %ls", *NewestCapturePath, *DestPath);

			if (PlatformFileSystem.FileExists(*DestPath))
			{
				PlatformFileSystem.DeleteFile(*DestPath);
			}

			if (PlatformFileSystem.CreateDirectoryTree(*FPaths::GetPath(DestPath)))
			{
				if (PlatformFileSystem.MoveFile(*DestPath, *NewestCapturePath))
				{
					bLaunchDestPath = true;
				}
				else
				{
					uint32 WriteErrorCode = FPlatformMisc::GetLastError();
					TCHAR WriteErrorBuffer[2048];
					FPlatformMisc::GetSystemErrorMessage(WriteErrorBuffer, 2048, WriteErrorCode);
					UE_LOGF(RenderDocPlugin, Warning, "Failed to move: %ls to %ls. WriteError: %u (%ls)", *NewestCapturePath, *DestPath, WriteErrorCode, WriteErrorBuffer);
				}
			}
			else
			{
				uint32 WriteErrorCode = FPlatformMisc::GetLastError();
				TCHAR WriteErrorBuffer[2048];
				FPlatformMisc::GetSystemErrorMessage(WriteErrorBuffer, 2048, WriteErrorCode);
				UE_LOGF(RenderDocPlugin, Warning, "Failed to create directory tree for: %ls. WriteError: %u (%ls)", *DestPath, WriteErrorCode, WriteErrorBuffer);
			}
		}

		if (Flags & IRenderCaptureProvider::ECaptureFlags_Launch)
		{	
			TGraphTask<FRenderDocAsyncGraphTask>::CreateTask().ConstructAndDispatchWhenReady(ENamedThreads::GameThread, [Plugin, DestPath, bLaunchDestPath]()
			{
				if (bLaunchDestPath)
				{
					Plugin->StartRenderDoc(DestPath);
				}
				else
				{
					Plugin->StartRenderDoc(FString());
				}
			});
		}
	}

private:
	static bool UE_GEmitDrawEvents_BeforeCapture;
};
bool FRenderDocFrameCapturer::UE_GEmitDrawEvents_BeforeCapture = false;

class FRenderDocDummyInputDevice : public IInputDevice
{
public:
	FRenderDocDummyInputDevice(FRenderDocPluginModule* InPlugin) : ThePlugin(InPlugin) { }
	virtual ~FRenderDocDummyInputDevice() { }

	/** Tick the interface (used for controlling full engine frame captures). */
	virtual void Tick(float DeltaTime) override
	{
		check(ThePlugin);
		ThePlugin->Tick(DeltaTime);
	}

	/** The remaining interfaces are irrelevant for this dummy input device. */
	virtual void SendControllerEvents() override { }
	virtual void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override { }
	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override { return(false); }
	virtual void SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override { }
	virtual void SetChannelValues(int32 ControllerId, const FForceFeedbackValues &values) override { }

private:
	FRenderDocPluginModule* ThePlugin;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FRenderDocPluginModule
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<class IInputDevice> FRenderDocPluginModule::CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
	UE_LOGF(RenderDocPlugin, Log, "Creating dummy input device (for intercepting engine ticks)");
	FRenderDocDummyInputDevice* InputDev = new FRenderDocDummyInputDevice(this);
	return MakeShareable((IInputDevice*)InputDev);
}

void FRenderDocPluginModule::StartupModule()
{
#if !UE_BUILD_SHIPPING // Disable in shipping builds
	Loader.Initialize();
	RenderDocAPI = nullptr;

	if (Loader.RenderDocAPI == nullptr)
	{
		return;
	}

	InjectDebugExecKeybind();

	// Regrettably, GUsingNullRHI is set to true AFTER the PostConfigInit modules
	// have been loaded (RenderDoc plugin being one of them). When this code runs
	// the following condition will never be true, so it must be tested again in
	// the Toolbar initialization code.
	if (GUsingNullRHI)
	{
		UE_LOGF(RenderDocPlugin, Display, "RenderDoc Plugin will not be loaded because a Null RHI (Cook Server, perhaps) is being used.");
		return;
	}

	// Obtain a handle to the RenderDoc DLL that has been loaded by the RenderDoc
	// Loader Plugin; no need for error handling here since the Loader would have
	// already handled and logged these errors (but check() them just in case...)
	RenderDocAPI = Loader.RenderDocAPI;
	check(RenderDocAPI);

	IModularFeatures::Get().RegisterModularFeature(IRenderCaptureProvider::GetModularFeatureName(), (IRenderCaptureProvider*)this);

	IModularFeatures::Get().RegisterModularFeature(IInputDeviceModule::GetModularFeatureName(), (IInputDeviceModule*)this);
	DelayedCaptureTick = 0;
	DelayedCaptureSeconds = 0.0;
	CaptureFrameCount = 0;
	CaptureEndTick = 0;
	bCaptureDelayInSeconds = false;
	bPendingCapture = false;
	bCaptureInProgress = false;
	bShouldCaptureAllActivity = false;
	CaptureFlags = 0;

	// Setup RenderDoc settings
	FString RenderDocCapturePath = FPaths::ProjectSavedDir() / TEXT("RenderDocCaptures");
	if (!IFileManager::Get().DirectoryExists(*RenderDocCapturePath))
	{
		IFileManager::Get().MakeDirectory(*RenderDocCapturePath, true);
	}

	FString CapturePath = FPaths::Combine(*RenderDocCapturePath, *FDateTime::Now().ToString());
	CapturePath = FPaths::ConvertRelativePathToFull(CapturePath);
	FPaths::NormalizeDirectoryName(CapturePath);
	
	RenderDocAPI->SetLogFilePathTemplate(TCHAR_TO_ANSI(*CapturePath));

	RenderDocAPI->SetFocusToggleKeys(nullptr, 0);
	RenderDocAPI->SetCaptureKeys(nullptr, 0);

	RenderDocAPI->SetCaptureOptionU32(eRENDERDOC_Option_CaptureCallstacks, CVarRenderDocCaptureCallstacks.GetValueOnAnyThread() ? 1 : 0);
	RenderDocAPI->SetCaptureOptionU32(eRENDERDOC_Option_RefAllResources, CVarRenderDocReferenceAllResources.GetValueOnAnyThread() ? 1 : 0);
	RenderDocAPI->SetCaptureOptionU32(eRENDERDOC_Option_SaveAllInitials, CVarRenderDocSaveAllInitials.GetValueOnAnyThread() ? 1 : 0);

	RenderDocAPI->MaskOverlayBits(eRENDERDOC_Overlay_None, eRENDERDOC_Overlay_None);

	static FAutoConsoleCommand CCmdRenderDocCaptureFrame = FAutoConsoleCommand(
		TEXT("renderdoc.CaptureFrame"),
		TEXT("Captures the rendering commands of the next frame and launches RenderDoc"),
		FConsoleCommandDelegate::CreateRaw(this, &FRenderDocPluginModule::CaptureFrame)
	);

#if WITH_EDITOR
	static FAutoConsoleCommand CCmdRenderDocCapturePIE = FAutoConsoleCommand(
		TEXT("renderdoc.CapturePIE"),
		TEXT("Starts a PIE session and captures the specified number of frames from the start."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FRenderDocPluginModule::CapturePIE)
	);
#endif // WITH_EDITOR

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FRenderDocPluginModule::OnPostEngineInit);

	UE_LOGF(RenderDocPlugin, Log, "RenderDoc plugin is ready!");
#endif // !UE_BUILD_SHIPPING
}

void FRenderDocPluginModule::OnPostEngineInit()
{
#if WITH_EDITOR
	if (FSlateApplication::IsInitialized() && !IsRunningCommandlet())
	{
		EditorExtension = MakeShared<FRenderDocPluginEditorExtension>(this);

		// Cannot register in StartupModule because the RDoc plugin is initialized earlier than the main frame's
		if (IMainFrameModule* MainFrameModule = FModuleManager::GetModulePtr<IMainFrameModule>("MainFrame"))
		{
			MainFrameModule->OnMainFrameCreationFinished().AddRaw(this, &FRenderDocPluginModule::OnMainFrameCreationFinished);
		}
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void FRenderDocPluginModule::OnMainFrameCreationFinished(TSharedPtr<SWindow> ParentWindow, bool bIsRunningStartupDialog)
{
	if (ParentWindow.IsValid())
	{
		MainWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
	}
}
#endif // WITH_EDITOR

void FRenderDocPluginModule::BeginFrameCapture()
{
	UE_LOGF(RenderDocPlugin, Log, "Capture frame and launch renderdoc!");
	ShowNotification(LOCTEXT("RenderDocBeginCaptureNotification", "RenderDoc capture started"), true);

	pRENDERDOC_SetCaptureOptionU32 SetOptions = Loader.RenderDocAPI->SetCaptureOptionU32;
	int ok = SetOptions(eRENDERDOC_Option_CaptureCallstacks, CVarRenderDocCaptureCallstacks.GetValueOnAnyThread() ? 1 : 0); check(ok);
		ok = SetOptions(eRENDERDOC_Option_RefAllResources, CVarRenderDocReferenceAllResources.GetValueOnAnyThread() ? 1 : 0); check(ok);
		ok = SetOptions(eRENDERDOC_Option_SaveAllInitials, CVarRenderDocSaveAllInitials.GetValueOnAnyThread() ? 1 : 0); check(ok);

	HWND WindowHandle = GetActiveWindow();

	typedef FRenderDocPluginLoader::RENDERDOC_API_CONTEXT RENDERDOC_API_CONTEXT;
	FRenderDocPluginModule* Plugin = this;
	FRenderDocPluginLoader::RENDERDOC_API_CONTEXT* RenderDocAPILocal = RenderDocAPI;
	ENQUEUE_RENDER_COMMAND(StartRenderDocCapture)(
		[Plugin, WindowHandle, RenderDocAPILocal](FRHICommandListImmediate& RHICmdList)
		{
			FRenderDocFrameCapturer::BeginFrameCapture(WindowHandle, RenderDocAPILocal);
		});
}

bool FRenderDocPluginModule::ShouldCaptureAllActivity() const
{
	// capturing more than 1 frame means that we can't just capture the current viewport : 
	return CVarRenderDocCaptureAllActivity.GetValueOnAnyThread() || (CVarRenderDocCaptureFrameCount.GetValueOnAnyThread() > 1);
}

void FRenderDocPluginModule::ShowNotification(const FText& Message, bool bForceNewNotification)
{
#if WITH_EDITOR
	FRenderDocPluginNotification::Get().ShowNotification(Message, bForceNewNotification);
#else // WITH_EDITOR
	GEngine->AddOnScreenDebugMessage((uint64)-1, 2.0f, FColor::Emerald, Message.ToString());
#endif // !WITH_EDITOR
}

void FRenderDocPluginModule::InjectDebugExecKeybind()
{
	// Inject our key bind into the debug execs
	GConfig->AddUniqueToSection(TEXT("/Script/Engine.PlayerInput"), TEXT("DebugExecBindings"), TEXT("(Key=F12,Command=\"RenderDoc.CaptureFrame\", Alt=true)"), GInputIni);
}

void FRenderDocPluginModule::EndFrameCapture(void* HWnd, uint32 Flags, const FString& DestFileName)
{
	HWND WindowHandle = (HWnd) ? reinterpret_cast<HWND>(HWnd) : GetActiveWindow();

	typedef FRenderDocPluginLoader::RENDERDOC_API_CONTEXT RENDERDOC_API_CONTEXT;
	FRenderDocPluginModule* Plugin = this;
	FRenderDocPluginLoader::RENDERDOC_API_CONTEXT* RenderDocAPILocal = RenderDocAPI;
	ENQUEUE_RENDER_COMMAND(EndRenderDocCapture)(
		[WindowHandle, RenderDocAPILocal, Plugin, Flags, DestFileName](FRHICommandListImmediate& RHICmdList)
		{
			FRenderDocFrameCapturer::EndFrameCapture(RHICmdList, WindowHandle, RenderDocAPILocal);
			FRenderDocFrameCapturer::SaveAndLaunch(Plugin, Flags, DestFileName);
		});

	DelayedCaptureTick = 0;
	DelayedCaptureSeconds = 0.0;
	CaptureFrameCount = 0;
	CaptureEndTick = 0;
	bPendingCapture = false;
	bCaptureInProgress = false;
}

void FRenderDocPluginModule::CaptureFrame(FViewport* InViewport, uint32 InFlags, FString const& InDestFileName)
{
	check(IsInGameThread());

	int32 FrameDelay = CVarRenderDocCaptureDelay.GetValueOnAnyThread();

	// Don't do anything if we're currently already waiting for a capture to end : 
	if (bCaptureInProgress)
	{
		return;
	}

	// In case there's no delay and we capture the current viewport, we can trigger the capture immediately : 
	bShouldCaptureAllActivity = ShouldCaptureAllActivity();
	if ((FrameDelay == 0) && !bShouldCaptureAllActivity)
	{
		DoFrameCaptureCurrentViewport(InViewport, InFlags, InDestFileName);
	}
	else
	{
		if (InViewport || !InDestFileName.IsEmpty() || (InFlags & ECaptureFlags_Launch))
		{
			UE_LOGF(RenderDocPlugin, Warning, "Deferred captures only support default params. Passed in params will be ignored.");
		}

		// store all CVars at beginning of capture in case they change while the capture is occurring : 
		CaptureFrameCount = CVarRenderDocCaptureFrameCount.GetValueOnAnyThread();
		bCaptureDelayInSeconds = CVarRenderDocCaptureDelayInSeconds.GetValueOnAnyThread() > 0;

		if (bCaptureDelayInSeconds)
		{
			DelayedCaptureSeconds = FPlatformTime::Seconds() + (double)FrameDelay;
		}
		else
		{
			// Begin tracking the global tick counter so that the Tick() method below can
			// identify the beginning and end of a complete engine update cycle.
			// NOTE: GFrameCounter counts engine ticks, while GFrameNumber counts render
			// frames. Multiple frames might get rendered in a single engine update tick.
			// All active windows are updated, in a round-robin fashion, within a single
			// engine tick. This includes thumbnail images for material preview, material
			// editor previews, cascade/persona previews, etc.
			DelayedCaptureTick = GFrameCounter + FrameDelay;
		}

		bPendingCapture = true;
	}
}

void FRenderDocPluginModule::CaptureFrame()
{
	CaptureFrame(nullptr, ECaptureFlags_Launch, FString());
}

#if WITH_EDITOR
void FRenderDocPluginModule::CapturePIE(const TArray<FString>& Args)
{
	if (Args.Num() < 1)
	{
		UE_LOGF(LogTemp, Error, "Usage: renderdoc.CapturePIE NumFrames");
		return;
	}

	int32 NumFrames = FCString::Atoi(*Args[0]);
	if (NumFrames < 1)
	{
		UE_LOGF(LogTemp, Error, "NumFrames must be greater than 0.");
		return;
	}

	void* PIEViewportHandle = GetActiveWindow();
	RenderDocAPI->SetActiveWindow(FRenderDocFrameCapturer::GetRenderdocDevicePointer(), PIEViewportHandle);
	RenderDocAPI->TriggerMultiFrameCapture(NumFrames);

	// Wait one frame before starting the PIE session, because RenderDoc will start capturing after the next
	// present, and we want to catch the first PIE frame.
	StartPIEDelayFrames = 1;
}
#endif // WITH_EDITOR

void FRenderDocPluginModule::DoFrameCaptureCurrentViewport(FViewport* InViewport, uint32 InFlags, const FString& InDestFileName)
{
	BeginFrameCapture();

	// infer the intended viewport to intercept/capture:
	FViewport* Viewport = InViewport;

	check(GEngine);
	if (!Viewport && GEngine->GameViewport)
	{
		check(GEngine->GameViewport->Viewport);
		if (GEngine->GameViewport->Viewport->HasFocus())
		{
			Viewport = GEngine->GameViewport->Viewport;
		}
	}

#if WITH_EDITOR
	if (!Viewport && GEditor)
	{
		// WARNING: capturing from a "PIE-Eject" Editor viewport will not work as
		// expected; in such case, capture via the console command
		// (this has something to do with the 'active' editor viewport when the UI
		// button is clicked versus the one which the console is attached to)
		Viewport = GEditor->GetActiveViewport();
	}
#endif // WITH_EDITOR

	check(Viewport);
	Viewport->Draw(true);

	EndFrameCapture(Viewport->GetWindow(), InFlags, InDestFileName);
}

void FRenderDocPluginModule::Tick(float DeltaTime)
{
#if WITH_EDITOR
	if (StartPIEDelayFrames > 0)
	{
		--StartPIEDelayFrames;
	}
	else if(StartPIEDelayFrames == 0)
	{
		UEditorEngine* EditorEngine = CastChecked<UEditorEngine>(GEngine);
		FRequestPlaySessionParams SessionParams;
		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		SessionParams.DestinationSlateViewport = LevelEditorModule.GetFirstActiveViewport();
		EditorEngine->RequestPlaySession(SessionParams);
		StartPIEDelayFrames = -1;
	}
#endif // WITH_EDITOR

	if (!bPendingCapture && !bCaptureInProgress)
		return;

	if (bPendingCapture)
	{
		check(!bCaptureInProgress); // can't be in progress and pending at the same time

		bool bStartCapturing = false;
		if (bCaptureDelayInSeconds)
		{
			bStartCapturing = FPlatformTime::Seconds() > DelayedCaptureSeconds;
		}
		else
		{
			const int64 TickDiff = GFrameCounter - DelayedCaptureTick;
			bStartCapturing = (TickDiff == 1);
		}

		if (bStartCapturing)
		{
			// are we capturing only the current viewport?
			if (!bShouldCaptureAllActivity)
			{
				DoFrameCaptureCurrentViewport(nullptr, ECaptureFlags_Launch, FString());
				check(!bCaptureInProgress && !bPendingCapture); // EndFrameCapture must have been called
			}
			else
			{
				BeginFrameCapture();
				// from now on, we'll detect the end of the capture by counting ticks : 
				CaptureEndTick = GFrameCounter + CaptureFrameCount + 1;
				bCaptureInProgress = true;
				bPendingCapture = false;
			}
		}
		else
		{
			float TimeLeft = bCaptureDelayInSeconds ? (float)(DelayedCaptureSeconds - FPlatformTime::Seconds()) : (float)(DelayedCaptureTick - GFrameCounter);
			const FText& SecondsOrFrames = bCaptureDelayInSeconds ? LOCTEXT("RenderDocSeconds", "seconds") : LOCTEXT("RenderDocFrames", "frames");

			ShowNotification(LOCGEN_FORMAT_ORDERED(LOCTEXT("RenderDocPendingCaptureNotification", "RenderDoc capture starting in {0} {1}"), TimeLeft, SecondsOrFrames), false);
		}
	}

	const int64 TickDiff = GFrameCounter - DelayedCaptureTick;
	if (bCaptureInProgress)
	{
		check(!bPendingCapture); // can't be in progress and pending at the same time

		if (GFrameCounter == CaptureEndTick)
		{
			EndFrameCapture(nullptr, ECaptureFlags_Launch, FString());
		}
		else
		{
			ShowNotification(LOCGEN_FORMAT_ORDERED(LOCTEXT("RenderDocCaptureInProgressNotification", "RenderDoc capturing frame #{0}"),
				CaptureFrameCount - (CaptureEndTick - 1 - GFrameCounter)), false);
		}
	}
}

void FRenderDocPluginModule::StartRenderDoc(FString LaunchPath)
{
	ShowNotification(LOCTEXT("RenderDocLaunchRenderDocNotification", "Launching RenderDoc GUI"), true);

	if (LaunchPath.IsEmpty())
	{
		LaunchPath = GetNewestCapture();
	}

	FString ArgumentString = FString::Printf(TEXT("\"%s\""), *LaunchPath);

	if (!LaunchPath.IsEmpty())
	{
		if (!RenderDocAPI->IsRemoteAccessConnected())
		{
			uint32 PID = RenderDocAPI->LaunchReplayUI(true, TCHAR_TO_ANSI(*ArgumentString));

			if (0 == PID)
			{
				UE_LOGF(LogTemp, Error, "Could not launch RenderDoc!!");
				ShowNotification(LOCTEXT("RenderDocLaunchRenderDocNotificationFailure", "Failed to launch RenderDoc GUI"), true);
			}
		}
	}

	ShowNotification(LOCTEXT("RenderDocLaunchRenderDocNotificationCompleted", "RenderDoc GUI Launched!"), true);
}

FString FRenderDocPluginModule::GetNewestCapture()
{
	char LogFile[512];
	uint64_t Timestamp;
	uint32_t LogPathLength = 512;
	uint32_t Index = 0;
	FString OutString;
	
	while (RenderDocAPI->GetCapture(Index, LogFile, &LogPathLength, &Timestamp))
	{
		OutString = FString::ConstructFromPtrSize(ANSI_TO_TCHAR(LogFile), LogPathLength);

		Index++;
	}
	
	return FPaths::ConvertRelativePathToFull(OutString);
}

void FRenderDocPluginModule::ShutdownModule()
{
	if (GUsingNullRHI)
		return;

	IModularFeatures::Get().UnregisterModularFeature(IRenderCaptureProvider::GetModularFeatureName(), (IRenderCaptureProvider*)this);

#if WITH_EDITOR
	if (IMainFrameModule* MainFrameModule = FModuleManager::GetModulePtr<IMainFrameModule>("MainFrame"))
	{
		MainFrameModule->OnMainFrameCreationFinished().RemoveAll(this);
	}

	EditorExtension.Reset();
#endif // WITH_EDITOR

	Loader.Release();

	RenderDocAPI = nullptr;
}

void FRenderDocPluginModule::BeginCapture(FRHICommandListImmediate* InRHICommandList, uint32 InFlags, FString const& InDestFileName)
{
	CaptureFlags = InFlags;
	CaptureFileName = InDestFileName;
	BeginCapture_RenderThread(InRHICommandList);
}

void FRenderDocPluginModule::EndCapture(FRHICommandListImmediate* InRHICommandList)
{
	EndCapture_RenderThread(InRHICommandList, CaptureFlags, CaptureFileName);
	CaptureFlags = 0;
	CaptureFileName.Reset();
}

void FRenderDocPluginModule::BeginCapture_RenderThread(FRHICommandListImmediate* InRHICommandList)
{
	RENDERDOC_DevicePointer Device = FRenderDocFrameCapturer::GetRenderdocDevicePointer();
	InRHICommandList->SubmitAndBlockUntilGPUIdle();
	InRHICommandList->EnqueueLambda([this, Device](FRHICommandListImmediate& RHICommandList)
	{
		RenderDocAPI->StartFrameCapture(Device, MainWindowHandle);
	});
}

void FRenderDocPluginModule::EndCapture_RenderThread(FRHICommandListImmediate* InRHICommandList, uint32 InFlags, FString const& InDestFileName)
{
	RENDERDOC_DevicePointer Device = FRenderDocFrameCapturer::GetRenderdocDevicePointer();
	InRHICommandList->SubmitAndBlockUntilGPUIdle();
	InRHICommandList->EnqueueLambda([this, Device, InFlags, DestFileName = InDestFileName](FRHICommandListImmediate& RHICommandList)
	{
		uint32 Result = RenderDocAPI->EndFrameCapture(Device, MainWindowHandle);
		if (Result == 1)
		{
			FRenderDocFrameCapturer::SaveAndLaunch(this, InFlags, DestFileName);
		}
	});
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FRenderDocPluginModule, RenderDocPlugin)
