// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRenderDocPlugin.h"
#include "RenderDocPluginLoader.h"

DECLARE_LOG_CATEGORY_EXTERN(RenderDocPlugin, Log, All);

class FRenderDocPluginEditorExtension;
class SWindow;

class FRenderDocPluginModule : public IRenderDocPlugin
{
	friend class FRenderDocDummyInputDevice;
	friend class FRenderDocFrameCapturer;

public:
	// Begin IRenderCaptureProvider interface.
	virtual bool CanSupportSubmissionThread() const { return false; }
	virtual void CaptureFrame(FViewport* InViewport, uint32 InFlags, FString const& InDestFileName) override;
	virtual void BeginCapture(FRHICommandListImmediate* InRHICommandList, uint32 InFlags, FString const& InDestFileName) override;
	virtual void EndCapture(FRHICommandListImmediate* InRHICommandList) override;
	// End IRenderCaptureProvider interface.

protected:
	// Begin IModuleInterface interface.
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End IModuleInterface interface.

	// Begin IInputDeviceModule interface.
	virtual TSharedPtr<class IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override;
	// End IInputDeviceModule interface.

private:
	void Tick(float DeltaTime);

	/** Injects a debug key bind into the local player so that the hot key works the same in game */
	void InjectDebugExecKeybind();

	/** Helper function for CVar command binding. */
	void CaptureFrame();

#if WITH_EDITOR
	void OnMainFrameCreationFinished(TSharedPtr<SWindow> ParentWindow, bool bIsRunningStartupDialog);
	void CapturePIE(const TArray<FString>& Args);
#endif // WITH_EDITOR

	void OnPostEngineInit();

	void DoFrameCaptureCurrentViewport(FViewport* InViewport, uint32 InFlags, FString const& InDestFileName);

	void BeginFrameCapture();
	void EndFrameCapture(void* HWnd, uint32 Flags, FString const& DestFileName);

	void BeginCapture_RenderThread(FRHICommandListImmediate* InRHICommandList);
	void EndCapture_RenderThread(FRHICommandListImmediate* InRHICommandList, uint32 InFlags, FString const& InDestFileName);

	bool ShouldCaptureAllActivity() const;
	FString GetNewestCapture();
	void ShowNotification(const FText& Message, bool bForceNewNotification);
	void StartRenderDoc(FString CapturePath);

private:
	FRenderDocPluginLoader Loader;
	FRenderDocPluginLoader::RENDERDOC_API_CONTEXT* RenderDocAPI;

	void* MainWindowHandle = nullptr; // Keep a handle of the main window so that programmatic captures (i.e. BeginCapture_RenderThread / EndCapture_RenderThread) can still pass a valid window handle to RenderDoc
	uint64 DelayedCaptureTick; // Tracks on which frame a delayed capture should trigger, if any (when bCaptureDelayInSeconds == false)
	double DelayedCaptureSeconds; // Tracks at which time a delayed capture should trigger, if any (when bCaptureDelayInSeconds == true)
	uint64 CaptureFrameCount; // Tracks how many frames should be captured
	uint64 CaptureEndTick; // Tracks the tick at which the capture currently in progress should end
	bool bCaptureDelayInSeconds:1; // Is the capture delay in seconds or ticks?
	bool bShouldCaptureAllActivity : 1; // true if all the whole frame should be captured, not just the active viewport
	bool bPendingCapture : 1; // true when a delayed capture has been triggered but hasn't started yet
	bool bCaptureInProgress:1; // true after BeginCapture() has been called and we're waiting for the end of the capture
	
	uint32 CaptureFlags; // Store the capture flags that are known at BeginCapture() for use at EndCapture()
	FString CaptureFileName; // Store the capture file name that is known at BeginCapture() for use at EndCapture()

#if WITH_EDITOR
	TSharedPtr<FRenderDocPluginEditorExtension> EditorExtension;
	int StartPIEDelayFrames = -1;
#endif // WITH_EDITOR
};
