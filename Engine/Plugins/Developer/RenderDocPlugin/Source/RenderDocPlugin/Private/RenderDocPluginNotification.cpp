// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "RenderDocPluginNotification.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"

FRenderDocPluginNotification::FRenderDocPluginNotification()
	: RenderDocNotificationPtr()
	, LastEnableTime(0.0)
{
}

void FRenderDocPluginNotification::ShowNotification(const FText& Message, bool bForceNewNotification)
{
	LastEnableTime = FPlatformTime::Seconds();

	// Just update the previous new notification if there's one : 
	if (RenderDocNotificationPtr.IsValid() && !bForceNewNotification)
	{
		RenderDocNotificationPtr.Pin()->SetText(Message);
	}
	else
	{
		// cancel the previous notification, if any, before starting out a new one : 
		if (RenderDocNotificationPtr.IsValid())
		{
			RenderDocNotificationPtr.Pin()->ExpireAndFadeout();
		}

		FNotificationInfo Info(Message);
		Info.bFireAndForget = false;

		// Setting fade out and expire time to 0 as the expire message is currently very obnoxious
		Info.FadeOutDuration = 1.0f;
		Info.ExpireDuration = 0.0f;

		RenderDocNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
		if (RenderDocNotificationPtr.IsValid())
		{
			RenderDocNotificationPtr.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
		}
	}
}

void FRenderDocPluginNotification::HideNotification()
{
	// Finished all requests! Notify the UI.
	TSharedPtr<SNotificationItem> NotificationItem = RenderDocNotificationPtr.Pin();

	if (NotificationItem.IsValid())
	{
		NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
		NotificationItem->ExpireAndFadeout();

		RenderDocNotificationPtr.Reset();
	}
}

void FRenderDocPluginNotification::Tick(float DeltaTime)
{
	const double OpenTime = 5.0;
	if (RenderDocNotificationPtr.IsValid() && (FPlatformTime::Seconds() - LastEnableTime) > OpenTime)
	{
		HideNotification();
	}
}

TStatId FRenderDocPluginNotification::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FRenderDocPluginNotification, STATGROUP_Tickables);
}

#endif //WITH_EDITOR
