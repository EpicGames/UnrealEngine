// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Framework/Notifications/NotificationManager.h"
#include "Tickable.h"

class FRenderDocPluginNotification : public FTickableGameObject
{
public:
	static FRenderDocPluginNotification& Get()
	{
		static FRenderDocPluginNotification Instance;
		return Instance;
	}

	void ShowNotification(const FText& Message, bool bForceNewNotification);
	void HideNotification();

protected:
	/** FTickableGameObject interface */
	virtual void Tick(float DeltaTime);
	virtual ETickableTickType GetTickableTickType() const { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override;
	virtual bool IsTickableWhenPaused() const override { return true; }
	virtual bool IsTickableInEditor() const override { return true; }

private:
	FRenderDocPluginNotification();
	FRenderDocPluginNotification(FRenderDocPluginNotification const& Notification);
	void operator=(FRenderDocPluginNotification const&);

	/** The source code symbol query in progress message */
	TWeakPtr<SNotificationItem> RenderDocNotificationPtr;
	double LastEnableTime;
};

#endif
