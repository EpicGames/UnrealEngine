// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderDocPluginSettings.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RenderDocPluginSettings)

static FName DeveloperSettingsConsoleVariableMetaFName(TEXT("ConsoleVariable"));

void URenderDocPluginSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (IsTemplate())
	{
		ImportConsoleVariableValues();
	}
#endif
}

FName URenderDocPluginSettings::GetCategoryName() const
{
	return FName(TEXT("Plugins"));
}

#if WITH_EDITOR
void URenderDocPluginSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
}
#endif
