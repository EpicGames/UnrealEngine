// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "RenderDocPluginStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "HAL/FileManager.h"
#include "Interfaces/IPluginManager.h"

FString FRenderDocPluginStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	auto myself = IPluginManager::Get().FindPlugin(TEXT("RenderDocPlugin"));
	check(myself.IsValid());
	static FString ContentDir = myself->GetBaseDir() / TEXT("Resources");
	return (ContentDir / RelativePath) + Extension;
}

TSharedPtr<FSlateStyleSet> FRenderDocPluginStyle::StyleSet = nullptr;
TSharedPtr<class ISlateStyle> FRenderDocPluginStyle::Get() { return StyleSet; }

void FRenderDocPluginStyle::Initialize()
{
	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet("RenderDocPluginStyle"));

	FString ProjectResourceDir = FPaths::ProjectPluginsDir() / TEXT("RenderDocPlugin/Resources");
	FString EngineResourceDir = FPaths::EnginePluginsDir() / TEXT("RenderDocPlugin/Resources");

	if (IFileManager::Get().DirectoryExists(*ProjectResourceDir)) //Is the plugin in the project? In that case, use those resources
	{
		StyleSet->SetContentRoot(ProjectResourceDir);
		StyleSet->SetCoreContentRoot(ProjectResourceDir);
	}
	else //Otherwise, use the global ones
	{
		StyleSet->SetContentRoot(EngineResourceDir);
		StyleSet->SetCoreContentRoot(EngineResourceDir);
	}

	StyleSet->Set("RenderDocPlugin.Icon", new FSlateImageBrush(FRenderDocPluginStyle::InContent("Icon40", ".png"), FVector2D(40.0f, 40.0f)));
	StyleSet->Set("RenderDocPlugin.CaptureFrame", new FSlateImageBrush(FRenderDocPluginStyle::InContent("ViewportIcon16", ".png"), FVector2D(16.0f, 16.0f)));
	
	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

void FRenderDocPluginStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

#endif //WITH_EDITOR
