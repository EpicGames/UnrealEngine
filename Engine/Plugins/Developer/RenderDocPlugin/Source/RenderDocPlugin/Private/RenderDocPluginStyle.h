// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Styling/ISlateStyle.h"

class FRenderDocPluginStyle
{
public:
	static void Initialize();
	static void Shutdown();
	static TSharedPtr<class ISlateStyle> Get();

private:
	static FString InContent(const FString& RelativePath, const ANSICHAR* Extension);

private:
	static TSharedPtr<class FSlateStyleSet> StyleSet;
};

#endif
