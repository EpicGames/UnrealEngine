// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "SRenderDocPluginEditorExtension.h"
#include "SRenderDocPluginHelpWindow.h"

#include "RenderDocPluginCommands.h"
#include "RenderDocPluginModule.h"
#include "RenderDocPluginSettings.h"
#include "RenderDocPluginStyle.h"

#include "Editor/EditorEngine.h"
#include "SEditorViewportToolBarMenu.h"
#include "SViewportToolBarComboMenu.h"
#include "Kismet2/DebuggerCommands.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Interfaces/IMainFrameModule.h"
#include "LevelEditor.h"
#include "RHI.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "ToolMenus.h"

extern UNREALED_API UEditorEngine* GEditor;

FRenderDocPluginEditorExtension::FRenderDocPluginEditorExtension(FRenderDocPluginModule* ThePlugin)
{
	Initialize(ThePlugin);
}

FRenderDocPluginEditorExtension::~FRenderDocPluginEditorExtension()
{
	FRenderDocPluginStyle::Shutdown();
	FRenderDocPluginCommands::Unregister();
	UToolMenus::UnregisterOwner(this);
}

void FRenderDocPluginEditorExtension::Initialize(FRenderDocPluginModule* ThePlugin)
{
	if (GUsingNullRHI)
	{
		UE_LOGF(RenderDocPlugin, Display, "RenderDoc Plugin will not be loaded because a Null RHI (Cook Server, perhaps) is being used.");
		return;
	}

	// The LoadModule request below will crash if running as an editor commandlet!
	check(!IsRunningCommandlet());

	FRenderDocPluginStyle::Initialize();
	FRenderDocPluginCommands::Register();

#if WITH_EDITOR
	if (!IsRunningGame())
	{
		FToolMenuOwnerScoped ScopedOwner(this);

		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.ViewportToolbar");
	
		FToolMenuSection& RightSection = Menu->FindOrAddSection("Right");
		FToolMenuEntry& Entry = RightSection.AddMenuEntry("RenderDocCaptureFrame", FRenderDocPluginCommands::Get().CaptureFrame);
		Entry.ToolBarData.LabelOverride = FText::GetEmpty();
		Entry.InsertPosition.Position = EToolMenuInsertType::First;
	}
#endif // WITH_EDITOR

	// Would be nice to use the preprocessor definition WITH_EDITOR instead, but the user may launch a standalone the game through the editor...
	if (GEditor != nullptr)
	{
		check(FPlayWorldCommands::GlobalPlayWorldActions.IsValid());

		//Register the editor hotkeys
		FPlayWorldCommands::GlobalPlayWorldActions->MapAction(FRenderDocPluginCommands::Get().CaptureFrame,
			FExecuteAction::CreateLambda([]()
		{
			FRenderDocPluginModule& PluginModule = FModuleManager::GetModuleChecked<FRenderDocPluginModule>("RenderDocPlugin");
			PluginModule.CaptureFrame(nullptr, IRenderCaptureProvider::ECaptureFlags_Launch, FString());
		}),
			FCanExecuteAction());

		const URenderDocPluginSettings* Settings = GetDefault<URenderDocPluginSettings>();
		if (Settings->bShowHelpOnStartup)
		{
			GEditor->EditorAddModalWindow(SNew(SRenderDocPluginHelpWindow));
		}
	}
}

#endif //WITH_EDITOR
