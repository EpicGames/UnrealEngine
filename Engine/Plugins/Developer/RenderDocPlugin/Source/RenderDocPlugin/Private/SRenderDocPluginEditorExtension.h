// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "SViewportToolBar.h"

class FRenderDocPluginModule;
class FExtensionBase;
class FExtensibilityManager;
class FExtender;
class FToolBarBuilder;

class FRenderDocPluginEditorExtension
{
public:
  FRenderDocPluginEditorExtension(FRenderDocPluginModule* ThePlugin);
  ~FRenderDocPluginEditorExtension();

private:
  void Initialize(FRenderDocPluginModule* ThePlugin);
};

#endif //WITH_EDITOR
