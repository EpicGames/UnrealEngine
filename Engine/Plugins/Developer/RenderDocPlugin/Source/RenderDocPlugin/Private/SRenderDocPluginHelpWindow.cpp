// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "SRenderDocPluginHelpWindow.h"
#include "RenderDocPluginStyle.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"
#include "UnrealClient.h"
#include "Editor/EditorEngine.h"
extern UNREALED_API UEditorEngine* GEditor;
#include "Editor.h"
#include "RenderDocPluginSettings.h"

#define LOCTEXT_NAMESPACE "RenderDocPlugin"

static TAutoConsoleVariable<int32> CVarRenderDocShowHelpOnStartup(
	TEXT("renderdoc.ShowHelpOnStartup"),
	0,
	TEXT("0 - Greeting has been shown and will not appear on startup. ")
	TEXT("1 - Greeting will be shown during next startup."));

static void OnBrowserLinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata, TSharedRef<SWidget> ParentWidget)
{
	const FString* UrlPtr = Metadata.Find(TEXT("href"));
	if (UrlPtr)
	{
		FPlatformProcess::LaunchURL(**UrlPtr, nullptr, nullptr);
	}
}

void SRenderDocPluginHelpWindow::Construct(const FArguments& InArgs)
{
	SWindow::Construct(SWindow::FArguments()
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.IsPopupWindow(false)
		.SizingRule(ESizingRule::Autosized)
		.SupportsTransparency(EWindowTransparency::None)
		.InitialOpacity(1.0f)
		.FocusWhenFirstShown(true)
		.bDragAnywhere(false)
		.ActivationPolicy(EWindowActivationPolicy::Always)
		.ScreenPosition(FVector2D((float)(GSystemResolution.ResX) / 2.0, (float)(GSystemResolution.ResY) / 2.0))
		[
			SNew(SGridPanel)
			.FillColumn(0, 0.2f)
			.FillColumn(1, 0.7f)
			.FillColumn(2, 0.1f)
			.FillRow(0, 0.9f)
			.FillRow(1, 0.1f)

			+SGridPanel::Slot(0, 0)
			.ColumnSpan(3)
			.Padding(20)
			[
				SNew(SRichTextBlock)
				.Text(LOCTEXT("HelpMessage", "<LargeText>Hello and thank you for trying out the UE RenderDoc plugin!</>\n\nThis plugin will allow you to capture rendering operations in the engine, and inspect them using RenderDoc.\nThere are three ways of capturing a frame:\n* You can press the <NormalText.Important>green capture button</> in the top right of any viewport.\n* You can run the <NormalText.Important>renderdoc.CaptureFrame</> console command. This will work anywhere and is very useful for capturing frames in packaged builds.\n* You can use the capture hotkey (<NormalText.Important>Alt+F12</>). Please note that the hotkey is only active in editor windows and Play-In-Editor sessions.\n\nIf you are having trouble with not getting enough data in your captures, consider checking out the capture settings.\nThey can be found under <NormalText.Important>[Edit]->[Project Settings...]->[Plugins/RenderDoc]</>. All settings have tooltips that detail what they do.\n\nA good place to start learning graphics debugging in UE is the <a id=\"browser\" href=\"https://docs.unrealengine.com/latest/INT/Programming/Rendering/ShaderDevelopment\" style=\"Hyperlink\">Epic Rendering FAQ</>\nIt contains information on what CVar/project settings you should set when debugging shaders to get access to as much data as possible.\n\nIf you have any questions or suggestions regarding the plugin, please contact me via email or my github page:\n<NormalText.Important>temaran (at) gmail (dot) com</>\n\nI would also like to give major shoutouts to BaldurK, the author of RenderDoc:\n<a id=\"browser\" href=\"https://github.com/baldurk\" style=\"Hyperlink\">https://github.com/baldurk</>\nAnd Slomp, who has made many great contributions to the project:\n<a id=\"browser\" href=\"https://github.com/slomp\" style=\"Hyperlink\">https://github.com/slomp</>"))
				.DecoratorStyleSet(&FAppStyle::Get())
				+SRichTextBlock::HyperlinkDecorator(TEXT("browser"), FSlateHyperlinkRun::FOnClick::CreateStatic(&OnBrowserLinkClicked, AsShared()))
			]

			+SGridPanel::Slot(2, 0)
			.Padding(20)
			[
				SNew(SBox)
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Right)
				[
					SNew(SImage)
					.Image(FRenderDocPluginStyle::Get()->GetBrush("RenderDocPlugin.Icon"))
				]
			]

			+SGridPanel::Slot(0, 1)
			[
				SNew(SBox)
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Left)
				[
					SNew(SCheckBox)
					.IsChecked_Lambda([&]()
					{
						const URenderDocPluginSettings* Settings = GetDefault<URenderDocPluginSettings>();
						return Settings->bShowHelpOnStartup ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([&](const ECheckBoxState NewState)
					{
						URenderDocPluginSettings* Settings = GetMutableDefault<URenderDocPluginSettings>();
						Settings->bShowHelpOnStartup = (NewState == ECheckBoxState::Checked);
					})
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ShowHelpOnStartupText", "Show on startup"))
					]
				]
			]

			+SGridPanel::Slot(2, 1)
			[
				SNew(SBox)
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.OnClicked(this, &SRenderDocPluginHelpWindow::Close)
					.Text(LOCTEXT("CloseButton", "Close"))
				]
			]
		]);

	bIsTopmostWindow = true;
	FlashWindow();
}

FReply SRenderDocPluginHelpWindow::Close()
{
	RequestDestroyWindow();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE

#endif //WITH_EDITOR
