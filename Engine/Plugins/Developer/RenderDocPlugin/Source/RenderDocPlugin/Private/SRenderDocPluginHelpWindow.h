// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"

#if WITH_EDITOR

class SRenderDocPluginHelpWindow : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SRenderDocPluginHelpWindow)
	{
	}
	SLATE_END_ARGS()

	SRenderDocPluginHelpWindow() {}

	/** Widget constructor */
	void Construct(const FArguments& Args);

private:
	FReply Close();
};

#endif
