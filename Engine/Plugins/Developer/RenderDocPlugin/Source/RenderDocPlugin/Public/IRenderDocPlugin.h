// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IInputDeviceModule.h"
#include "IRenderCaptureProvider.h"

/**
 * The RenderDoc plugin works as an input plugin. Regular plugins built upon
 * IModuleInterface lack the ability of ticking, while IInputDeviceModule instantiates and
 * manages an IInputDevice object that is capable of ticking.
 *
 * By responding to tick events, the RenderDoc plugin is able to intercept the entire frame
 * activity, including Editor (Slate) UI rendering and SceneCapture updates.
 */
class IRenderDocPlugin : public IInputDeviceModule, public IRenderCaptureProvider
{
public:
	static inline IRenderDocPlugin& Get()
	{
		return FModuleManager::LoadModuleChecked<IRenderDocPlugin>("RenderDocPlugin");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("RenderDocPlugin");
	}
};
