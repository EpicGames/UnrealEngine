// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "RenderDocPluginSettings.generated.h"

#define UE_API RENDERDOCPLUGIN_API

UCLASS(MinimalAPI, config = Engine, defaultconfig, meta = (DisplayName = "RenderDoc"))
class URenderDocPluginSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(config, EditAnywhere, Category = "Frame Capture Settings", meta = (
		ConsoleVariable = "renderdoc.CaptureAllActivity", DisplayName = "Capture all activity",
		ToolTip = "If checked, RenderDoc will capture all activity in all viewports and editor windows for the entire frame instead of only the current viewport.",
		ConfigRestartRequired = false))
	uint32 bCaptureAllActivity : 1;

	UPROPERTY(config, EditAnywhere, Category = "Frame Capture Settings", meta = (
		ConsoleVariable = "renderdoc.CaptureCallstacks", DisplayName = "Capture all call stacks",
		ToolTip = "If checked, RenderDoc will capture callstacks for all API calls.",
		ConfigRestartRequired = false))
	uint32 bCaptureAllCallstacks : 1;

	UPROPERTY(config, EditAnywhere, Category = "Frame Capture Settings", meta = (
		ConsoleVariable = "renderdoc.ReferenceAllResources", DisplayName = "Reference all resources",
		ToolTip = "If checked, RenderDoc will include all rendering resources in the capture, even those that have not been used during the frame. Please note that doing this will significantly increase capture size.",
		ConfigRestartRequired = false))
	uint32 bReferenceAllResources : 1;

	UPROPERTY(config, EditAnywhere, Category = "Frame Capture Settings", meta = (
		ConsoleVariable = "renderdoc.SaveAllInitials", DisplayName = "Save all initial states",
		ToolTip = "If checked, RenderDoc will always capture the initial state of all rendering resources even if they are not likely to be used during the frame. Please note that doing this will significantly increase capture size.",
		ConfigRestartRequired = false))
	uint32 bSaveAllInitials : 1;

	UPROPERTY(config, EditAnywhere, Category = "Frame Capture Settings", meta = (
		ConsoleVariable = "renderdoc.CaptureDelayInSeconds", DisplayName = "Capture delay in seconds",
		ToolTip = "If checked, the capture delay's unit will be in seconds instead of frames.",
		ConfigRestartRequired = false))
		uint32 bCaptureDelayInSeconds : 1;

	UPROPERTY(config, EditAnywhere, Category = "Frame Capture Settings", meta = (
		ConsoleVariable = "renderdoc.CaptureDelay", DisplayName = "Capture delay",
		ToolTip = "If > 0, RenderDoc will trigger the capture only after this amount of frames/seconds has passed.",
		ClampMin = 0,
		ConfigRestartRequired = false))
	int32 CaptureDelay;

	UPROPERTY(config, EditAnywhere, Category = "Frame Capture Settings", meta = (
		ConsoleVariable = "renderdoc.CaptureFrameCount", DisplayName = "Capture frame count",
		ToolTip = "If > 1, the RenderDoc capture will encompass more than a single frame. Note: this implies that all activity in all viewports and editor windows will be captured (i.e. same as CaptureAllActivity)",
		ClampMin = 1,
		ConfigRestartRequired = false))
	int32 CaptureFrameCount;

	UPROPERTY(config, EditAnywhere, Category = "Advanced Settings", meta = (
		ConsoleVariable = "renderdoc.AutoAttach", DisplayName = "Auto attach on startup",
		ToolTip = "If checked, then RenderDoc will attach on startup. Otherwise the command line argument '-AttachRenderDoc' is required."
		))
	uint32 bAutoAttach : 1;

	UPROPERTY(config, EditAnywhere, Category = "Advanced Settings", meta = (
		ConsoleVariable = "renderdoc.ShowHelpOnStartup", DisplayName = "Show help on startup",
		ToolTip = "If checked, a help window will be shown on startup.",
		ConfigRestartRequired = true))
	uint32 bShowHelpOnStartup : 1;

	UPROPERTY(config, EditAnywhere, Category = "Advanced Settings", meta = (
		ConsoleVariable = "renderdoc.EnableCrashHandler", DisplayName = "Use the RenderDoc crash handler",
		ToolTip = "If checked, the RenderDoc crash handler will be used if a crash occurs (Only use this if you know the problem is with RenderDoc and you want to notify the RenderDoc developers!).",
		ConfigRestartRequired = true))
	uint32 bEnableRenderDocCrashHandler : 1;

	UPROPERTY(config, EditAnywhere, Category = "Advanced Settings", meta = (
		ConsoleVariable = "renderdoc.BinaryPath", DisplayName = "RenderDoc executable path",
		ToolTip = "Path to the main RenderDoc executable to use.",
		ConfigRestartRequired = true))
	FString RenderDocBinaryPath;

public:
	UE_API virtual void PostInitProperties() override;
	UE_API virtual FName GetCategoryName() const;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override; 
#endif
};

#undef UE_API
