// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	using System.IO;

	public class RenderDocPlugin : ModuleRules
	{
		public RenderDocPlugin(ReadOnlyTargetRules Target) : base(Target)
        {
			PrivateIncludePathModuleNames.AddRange([
				"VulkanRHI",
			]);

			if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
			{
				PrivateIncludePathModuleNames.AddRange([
					"D3D11RHI",
					"D3D12RHI"
				]);
			}

			PublicDependencyModuleNames.AddRange(new string[]
			{
				"Core"
				, "CoreUObject"
				, "Engine"
				, "InputCore"
				, "DesktopPlatform"
				, "Projects"
				, "RenderCore"
				, "InputDevice"
				, "RHI"				// RHI module: required for accessing the UE flag GUsingNullRHI.
				, "DeveloperSettings"
			});

			if (Target.bBuildEditor == true)
			{
				DynamicallyLoadedModuleNames.AddRange(new string[] { "LevelEditor" });
				PublicDependencyModuleNames.AddRange(new string[]
				{
					"Slate"
					, "SlateCore"
					, "EditorFramework"
					, "UnrealEd"
					, "MainFrame"
					, "GameProjectGeneration"
					, "ToolMenus"
				});
			}

            AddEngineThirdPartyPrivateStaticDependencies(Target, "RenderDoc");
        }
	}
}
